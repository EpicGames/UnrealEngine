// Copyright Epic Games, Inc. All Rights Reserved.


#include "WebSocketDeserializedMessage.h"
#include "JsonObjectConverter.h"
#include "UObject/CoreRedirects.h"
#include "WebSocketMessagingModule.h"

FWebSocketDeserializedMessage::FWebSocketDeserializedMessage()
	: Expiration(FDateTime::MaxValue())	// Make sure messages don't expire if no expiration is specified.
	, Message(nullptr)
	, Scope(EMessageScope::All)
{
}

FWebSocketDeserializedMessage::~FWebSocketDeserializedMessage()
{
	if (Message)
	{
		FMemory::Free(Message);
	}
}

bool FWebSocketDeserializedMessage::ParseJson(const FString& InJson, FString& OutParseError)
{
	static TMap<FString, EMessageScope> MessageScopeStringMapping =
	{
		{"Thread", EMessageScope::Thread},
		{ "Process", EMessageScope::Process },
		{ "Network", EMessageScope::Network },
		{ "All", EMessageScope::All }
	};

	TSharedPtr<FJsonValue> RootValue;

	const TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(InJson);
	if (FJsonSerializer::Deserialize(JsonReader, RootValue))
	{
		const TSharedPtr<FJsonObject> RootObject = RootValue->AsObject();
		if (!RootObject)
		{
			return false;
		}

		FString MessageType;
		if (!RootObject->TryGetStringField(WebSocketMessaging::Tag::MessageType, MessageType))
		{
			OutParseError = FString::Printf(TEXT("Missing Mendatory Field: \"%s\"."), WebSocketMessaging::Tag::MessageType);
			return false;
		}

		FString JsonSender;
		if (!RootObject->TryGetStringField(WebSocketMessaging::Tag::Sender, JsonSender))
		{
			OutParseError = FString::Printf(TEXT("Missing Mendatory Field: \"%s\"."), WebSocketMessaging::Tag::Sender);
			return false;
		}

		if (!FMessageAddress::Parse(JsonSender, Sender))
		{
			OutParseError = FString::Printf(TEXT("Field \"%s\": \"%s\" is not a valid Message Address."), WebSocketMessaging::Tag::Sender, *JsonSender);
			return false;
		}

		const TSharedPtr<FJsonObject>* JsonAnnotations = nullptr;
		if (RootObject->TryGetObjectField(WebSocketMessaging::Tag::Annotations, JsonAnnotations))
		{
			for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : (*JsonAnnotations)->Values)
			{
				Annotations.Add(*Pair.Key, Pair.Value->AsString());
			}
		}

		const TSharedPtr<FJsonObject>* JsonMessage = nullptr;
		if (!RootObject->TryGetObjectField(WebSocketMessaging::Tag::Message, JsonMessage))
		{
			OutParseError = FString::Printf(TEXT("Missing Mendatory Field: \"%s\"."), WebSocketMessaging::Tag::Message);
			return false;
		}

		UScriptStruct* ScriptStruct = FindObjectSafe<UScriptStruct>(nullptr, *MessageType);
		
		if (!ScriptStruct)
		{
			const FCoreRedirectObjectName OldObjectName(MessageType);
			const FCoreRedirectObjectName NewObjectName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Struct, OldObjectName);
			if (NewObjectName.IsValid() && OldObjectName != NewObjectName)
			{
				ScriptStruct = FindObject<UScriptStruct>(nullptr, *NewObjectName.ToString());
			}
		}
		
		if (!ScriptStruct)
		{
			OutParseError = FString::Printf(TEXT("Field \"%s\": The message type \"%s\" is not a valid UScriptStruct."), WebSocketMessaging::Tag::MessageType, *MessageType);
			return false;
		}

		TypeInfo = ScriptStruct;

		if (Message)
		{
			FMemory::Free(Message);
		}

		Message = FMemory::Malloc(ScriptStruct->GetStructureSize());
		ScriptStruct->InitializeStruct(Message);

		if (!FJsonObjectConverter::JsonObjectToUStruct(JsonMessage->ToSharedRef(), ScriptStruct, Message))
		{
			FMemory::Free(Message);
			Message = nullptr;
			OutParseError = FString::Printf(TEXT("Failed to deserialize UStruct \"%s\" from message data."), *MessageType);
			return false;
		}

		int64 UnixTime;
		if (RootObject->TryGetNumberField(WebSocketMessaging::Tag::Expiration, UnixTime))
		{
			Expiration = FDateTime::FromUnixTimestamp(UnixTime);
		}

		if (RootObject->TryGetNumberField(WebSocketMessaging::Tag::TimeSent, UnixTime))
		{
			TimeSent = FDateTime::FromUnixTimestamp(UnixTime);
		}

		FString ScopeString;
		if (RootObject->TryGetStringField(WebSocketMessaging::Tag::Scope, ScopeString))
		{
			if (MessageScopeStringMapping.Contains(ScopeString))
			{
				Scope = MessageScopeStringMapping[ScopeString];
			}
			// unknown scope string
			else
			{
				OutParseError = FString::Printf(TEXT("Field \"%s\": Unknown scope string: \"%s\"."), WebSocketMessaging::Tag::Scope, *ScopeString);
				return false;
			}
		}

		TArray<FString> RecipientsStrings;
		if (RootObject->TryGetStringArrayField(WebSocketMessaging::Tag::Recipients, RecipientsStrings))
		{
			for (const FString& RecipientString : RecipientsStrings)
			{
				FMessageAddress RecipientAddress;
				if (FMessageAddress::Parse(RecipientString, RecipientAddress))
				{
					Recipients.Add(RecipientAddress);
				}
			}
		}

		return true;
	}

	OutParseError = TEXT("Message is not a valid json format.");
	return false;
}
