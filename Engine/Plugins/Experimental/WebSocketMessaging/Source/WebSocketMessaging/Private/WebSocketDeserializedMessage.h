// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "IMessageAttachment.h"
#include "IMessageContext.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FWebSocketDeserializedMessage : public IMessageContext
{
public:
	FWebSocketDeserializedMessage();

	virtual ~FWebSocketDeserializedMessage() override;

	bool ParseJson(const FString& InJson, FString& OutParseError);

	virtual TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> GetAttachment() const override
	{
		return nullptr;
	}

	virtual const FDateTime& GetExpiration() const override
	{
		return Expiration;
	}

	virtual const void* GetMessage() const override
	{
		return Message;
	}

	virtual const TWeakObjectPtr<UScriptStruct, FWeakObjectPtr>& GetMessageTypeInfo() const override
	{
		return TypeInfo;
	}

	virtual const FMessageAddress& GetSender() const override
	{
		return Sender;
	}

	virtual const FDateTime& GetTimeSent() const override
	{
		return TimeSent;
	}

	virtual const TMap<FName, FString>& GetAnnotations() const override
	{
		return Annotations;
	}

	virtual TSharedPtr<IMessageContext, ESPMode::ThreadSafe> GetOriginalContext() const override
	{
		return nullptr;
	}

	virtual EMessageFlags GetFlags() const override
	{
		return EMessageFlags::None;
	}

	virtual const FMessageAddress& GetForwarder() const override
	{
		return Sender;
	}

	virtual ENamedThreads::Type GetSenderThread() const override
	{
		return ENamedThreads::AnyThread;
	}

	virtual const TArray<FMessageAddress, FDefaultAllocator>& GetRecipients() const override
	{
		return Recipients;
	}

	virtual const FDateTime& GetTimeForwarded() const override
	{
		return TimeSent;
	}

	virtual EMessageScope GetScope() const override
	{
		return Scope;
	}

protected:
	FDateTime Expiration;
	FDateTime TimeSent;
	void* Message;
	TWeakObjectPtr<UScriptStruct> TypeInfo;
	FMessageAddress Sender;
	TMap<FName, FString> Annotations;
	TArray<FMessageAddress> Recipients;
	EMessageScope Scope;
};
