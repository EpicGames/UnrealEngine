// Copyright Epic Games, Inc. All Rights Reserved.


#include "WebSocketMessageTransport.h"
#include "Backends/CborStructSerializerBackend.h"
#include "CborReader.h"
#include "CborWriter.h"
#include "IMessageContext.h"
#include "IMessageTransportHandler.h"
#include "INetworkingWebSocket.h"
#include "IWebSocket.h"
#include "IWebSocketNetworkingModule.h"
#include "IWebSocketServer.h"
#include "JsonObjectConverter.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"
#include "StructSerializer.h"
#include "WebSocketDeserializedMessage.h"
#include "WebSocketMessagingModule.h"
#include "WebSocketMessagingSettings.h"
#include "WebSocketsModule.h"

bool FWebSocketMessageConnection::IsConnected() const
{
	if (WebSocketConnection)
	{
		return WebSocketConnection->IsConnected();
	}

	return WebSocketServerConnection != nullptr;
}

void FWebSocketMessageConnection::Close()
{
	if (WebSocketConnection)
	{
		return WebSocketConnection->Close();
	}
}

FWebSocketMessageTransport::FWebSocketMessageTransport()
{
}

FWebSocketMessageTransport::~FWebSocketMessageTransport()
{
}

FName FWebSocketMessageTransport::GetDebugName() const
{
	static const FName DebugName("WebSocketMessageTransport");
	return DebugName;
}

bool FWebSocketMessageTransport::StartTransport(IMessageTransportHandler& InHandler)
{
	const UWebSocketMessagingSettings* Settings = GetDefault<UWebSocketMessagingSettings>();
	
	TransportHandler = &InHandler;
	
	const int32 ServerPort = Settings->GetServerPort();
	
	// Cache the settings to be able to detect changes.
	LastServerPort = ServerPort;
	LastServerBindAddress = Settings->ServerBindAddress;
	LastConnectionEndpoints = Settings->ConnectToEndpoints;
	LastHttpHeaders = Settings->HttpHeaders;

	FString ServerBindAddress = Settings->ServerBindAddress;

	if (ServerBindAddress.Compare(TEXT("0.0.0.0")) == 0
		|| ServerBindAddress.Compare(TEXT("any"), ESearchCase::IgnoreCase) == 0)
	{
		ServerBindAddress = TEXT("");	// Leaving empty will bind to all adapters.
	}

	if (ServerPort > 0)
	{
		IWebSocketNetworkingModule* WebSocketNetworkingModule = FModuleManager::Get().LoadModulePtr<IWebSocketNetworkingModule>(TEXT("WebSocketNetworking"));
		if (WebSocketNetworkingModule)
		{
			Server = WebSocketNetworkingModule->CreateServer();
			if (Server)
			{
				FWebSocketClientConnectedCallBack Callback;
				Callback.BindThreadSafeSP(this, &FWebSocketMessageTransport::ClientConnected);
				
				if (!Server->Init(ServerPort, Callback, ServerBindAddress))
				{
					Server.Reset();
					UE_LOGF(LogWebSocketMessaging, Error, "Unable to start WebSocketMessaging Server on port %d", ServerPort);
				}
				else
				{
					ServerTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateThreadSafeSP(this, &FWebSocketMessageTransport::ServerTick));
					UE_LOGF(LogWebSocketMessaging, Log, "WebSocketMessaging Server started on port %d", ServerPort);
				}
			}
		}
		else
		{
			UE_LOGF(LogWebSocketMessaging, Log, "Unable to load WebSocketNetworking module, ensure to enable it");
		}
	}

	for (const FString& Url : Settings->ConnectToEndpoints)
	{
		FGuid Guid = FGuid::NewGuid();

		TMap<FString, FString> Headers;
		Headers.Add(WebSocketMessaging::Header::TransportId, Guid.ToString());
		for (const TPair<FString, FString>& Pair : Settings->HttpHeaders)
		{
			Headers.Add(Pair.Key, Pair.Value);
		}

		TSharedRef<IWebSocket, ESPMode::ThreadSafe> WebSocketConnection = FWebSocketsModule::Get().CreateWebSocket(Url, FString(), Headers);

		FWebSocketMessageConnectionRef WebSocketMessageConnection = MakeShared<FWebSocketMessageConnection>(Url, Guid, WebSocketConnection);

		WebSocketConnection->OnMessage().AddThreadSafeSP(this, &FWebSocketMessageTransport::OnJsonMessage, WebSocketMessageConnection);
		WebSocketConnection->OnClosed().AddThreadSafeSP(this, &FWebSocketMessageTransport::OnClosed, WebSocketMessageConnection);
		WebSocketConnection->OnConnected().AddThreadSafeSP(this, &FWebSocketMessageTransport::OnConnected, WebSocketMessageConnection);
		WebSocketConnection->OnConnectionError().AddThreadSafeSP(this, &FWebSocketMessageTransport::OnConnectionError, WebSocketMessageConnection);

		WebSocketMessageConnections.Add(Guid, WebSocketMessageConnection);

		WebSocketConnection->Connect();
	}

	return true;
}

void FWebSocketMessageTransport::StopTransport()
{
	FTSTicker::RemoveTicker(ServerTickerHandle);
	
	if (Server.IsValid())
	{
		Server.Reset();
	}

	for (TPair<FGuid, FWebSocketMessageConnectionRef> Pair : WebSocketMessageConnections)
	{
		Pair.Value->bDestroyed = true;
		Pair.Value->Close();
	}
	WebSocketMessageConnections.Empty();
}

bool FWebSocketMessageTransport::NeedsRestart() const
{
	const UWebSocketMessagingSettings* Settings = GetDefault<UWebSocketMessagingSettings>();
	
	if (LastServerPort != Settings->GetServerPort()
		|| LastServerBindAddress != Settings->ServerBindAddress
		|| LastConnectionEndpoints != Settings->ConnectToEndpoints
		|| !LastHttpHeaders.OrderIndependentCompareEqual(Settings->HttpHeaders))
	{
		return true;
	}

	return false;
}

void FWebSocketMessageTransport::OnClosed(int32 InCode, const FString& InReason, bool bInUserClose, FWebSocketMessageConnectionRef InWebSocketMessageConnection)
{
	UE_LOGF(LogWebSocketMessaging, Log, "Connection to %ls closed, Code: %d Reason: \"%ls\" UserClose: %ls, retrying...", *InWebSocketMessageConnection->Url, InCode, *InReason, bInUserClose ? TEXT("true") : TEXT("false"));
	ForgetTransportNode(InWebSocketMessageConnection);
	InWebSocketMessageConnection->bIsConnecting = false;
	RetryConnection(InWebSocketMessageConnection);
}

void FWebSocketMessageTransport::OnConnectionError(const FString& InMessage, FWebSocketMessageConnectionRef InWebSocketMessageConnection)
{
	if (!InWebSocketMessageConnection->bIsConnecting)
	{
		UE_LOGF(LogWebSocketMessaging, Log, "Connection to %ls error: %ls, retrying...", *InWebSocketMessageConnection->Url, *InMessage);
	}
	ForgetTransportNode(InWebSocketMessageConnection);
	InWebSocketMessageConnection->bIsConnecting = false;
	RetryConnection(InWebSocketMessageConnection);
}

void FWebSocketMessageTransport::OnJsonMessage(const FString& InMessage, FWebSocketMessageConnectionRef InWebSocketMessageConnection)
{
	FString ParseError;
	const TSharedRef<FWebSocketDeserializedMessage> Context = MakeShared<FWebSocketDeserializedMessage>();
	if (Context->ParseJson(InMessage, ParseError))
	{
		TransportHandler->ReceiveTransportMessage(Context, InWebSocketMessageConnection->Guid);
	}
	else
	{
		UE_LOGF(LogWebSocketMessaging, Log, "Invalid Json Message received on %ls: %ls", *InWebSocketMessageConnection->Url, *ParseError);
	}
}

void FWebSocketMessageTransport::OnServerJsonMessage(void* InData, int32 InDataSize, FWebSocketMessageConnectionRef InWebSocketMessageConnection)
{
	FString Message = FString::ConstructFromPtrSize(reinterpret_cast<UTF8CHAR*>(InData), InDataSize);
	OnJsonMessage(Message, InWebSocketMessageConnection);
}

class FWebSocketMessageTransportSerializeHelper
{
public:
	static const TMap<EMessageScope, FString> MessageScopeStringMapping;

	static bool Serialize(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext, bool bInStandardizeCase, FString& OutJsonMessage)
	{
		TSharedRef<FJsonObject> JsonRoot = MakeShared<FJsonObject>();
		JsonRoot->SetStringField(WebSocketMessaging::Tag::Sender, InContext->GetSender().ToString());
		TArray<TSharedPtr<FJsonValue>> JsonRecipients;
		for (const FMessageAddress& Recipient : InContext->GetRecipients())
		{
			JsonRecipients.Add(MakeShared<FJsonValueString>(Recipient.ToString()));
		}
		JsonRoot->SetArrayField(WebSocketMessaging::Tag::Recipients, JsonRecipients);
		JsonRoot->SetStringField(WebSocketMessaging::Tag::MessageType, InContext->GetMessageTypePathName().ToString());
		JsonRoot->SetNumberField(WebSocketMessaging::Tag::Expiration, InContext->GetExpiration().ToUnixTimestamp());
		JsonRoot->SetNumberField(WebSocketMessaging::Tag::TimeSent, InContext->GetTimeSent().ToUnixTimestamp());
		JsonRoot->SetStringField(WebSocketMessaging::Tag::Scope, MessageScopeStringMapping[InContext->GetScope()]);


		TSharedRef<FJsonObject> JsonAnnotations = MakeShared<FJsonObject>();
		for (const TPair<FName, FString>& Pair : InContext->GetAnnotations())
		{
			JsonAnnotations->SetStringField(Pair.Key.ToString(), Pair.Value);
		}
		JsonRoot->SetObjectField(WebSocketMessaging::Tag::Annotations, JsonAnnotations);

		TSharedRef<FJsonObject> OutJsonObject = MakeShared<FJsonObject>();

		constexpr int64 CheckFlags = 0;
		constexpr int64 SkipFlags = 0;
		const EJsonObjectConversionFlags ConversionFlags = bInStandardizeCase ? EJsonObjectConversionFlags::None : EJsonObjectConversionFlags::SkipStandardizeCase;
		if (!FJsonObjectConverter::UStructToJsonObject(InContext->GetMessageTypeInfo().Get(), InContext->GetMessage(), OutJsonObject,
			CheckFlags, SkipFlags, nullptr, ConversionFlags))
		{
			return false;
		}

		JsonRoot->SetObjectField(WebSocketMessaging::Tag::Message, OutJsonObject);

		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutJsonMessage);
		return FJsonSerializer::Serialize(JsonRoot, Writer);
	}

	static bool Serialize(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext, bool bInStandardizeCase, FArrayWriter& OutCborBinaryWriter)
	{
		FCborHeader Header(ECborCode::Map | ECborCode::Indefinite);
		OutCborBinaryWriter << Header;
			
		{
			FCborWriter CborWriter(&OutCborBinaryWriter);
			
			CborWriter.WriteValue(FString(WebSocketMessaging::Tag::Sender));
			CborWriter.WriteValue(InContext->GetSender().ToString());
			
			CborWriter.WriteValue(FString(WebSocketMessaging::Tag::Recipients));
			CborWriter.WriteContainerStart(ECborCode::Array, -1);
			for (const FMessageAddress& Recipient : InContext->GetRecipients())
			{
				CborWriter.WriteValue(Recipient.ToString());
			}
			CborWriter.WriteContainerEnd();
			
			CborWriter.WriteValue(FString(WebSocketMessaging::Tag::MessageType));
			CborWriter.WriteValue(InContext->GetMessageTypePathName().ToString());
			
			CborWriter.WriteValue(FString(WebSocketMessaging::Tag::Expiration));
			CborWriter.WriteValue(InContext->GetExpiration().ToUnixTimestamp());
			
			CborWriter.WriteValue(FString(WebSocketMessaging::Tag::TimeSent));
			CborWriter.WriteValue(InContext->GetTimeSent().ToUnixTimestamp());
			
			CborWriter.WriteValue(FString(WebSocketMessaging::Tag::Scope));
			CborWriter.WriteValue(MessageScopeStringMapping[InContext->GetScope()]);
			
			
			CborWriter.WriteValue(FString(WebSocketMessaging::Tag::Annotations));
			CborWriter.WriteContainerStart(ECborCode::Map, -1);
			for (const TPair<FName, FString>& Annotation : InContext->GetAnnotations())
			{
				CborWriter.WriteValue(Annotation.Key.ToString());
				CborWriter.WriteValue(Annotation.Value);
			}
			CborWriter.WriteContainerEnd();
			
			CborWriter.WriteValue(FString(WebSocketMessaging::Tag::Message));
		}
		FCborStructSerializerBackend Backend(OutCborBinaryWriter, EStructSerializerBackendFlags::Default);
		FStructSerializer::Serialize(InContext->GetMessage(), *InContext->GetMessageTypeInfo().Get(), Backend);
			
		Header.Set(ECborCode::Break);
		OutCborBinaryWriter << Header;

		return true;
	}

	template<typename OutputType>
	struct TOnDemandSerializer
	{
		OutputType OutputMessage;
		bool bIsSerializeAttempted = false;
		bool bIsSerialized = false;
		bool bStandardizeCase = true;

		bool SerializeOnDemand(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
		{
			if (!bIsSerializeAttempted)
			{
				bIsSerializeAttempted = true;
				bIsSerialized = FWebSocketMessageTransportSerializeHelper::Serialize(InContext, bStandardizeCase, OutputMessage);
			}
			return bIsSerialized;
		}
	};
};

const TMap<EMessageScope, FString> FWebSocketMessageTransportSerializeHelper::MessageScopeStringMapping =
{
	{EMessageScope::Thread, "Thread"},
	{EMessageScope::Process, "Process"},
	{EMessageScope::Network, "Network"},
	{EMessageScope::All, "All"}
};

bool FWebSocketMessageTransport::TransportMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext, const TArray<FGuid>& InRecipients)
{
	TMap<FGuid, FWebSocketMessageConnectionRef> RecipientConnections;

	if (InRecipients.Num() == 0)
	{
		// broadcast the message to all valid connections
		RecipientConnections = WebSocketMessageConnections.FilterByPredicate([](const TPair<FGuid, FWebSocketMessageConnectionRef>& Pair) -> bool
			{
				return !Pair.Value->bDestroyed && Pair.Value->IsConnected();
			});
	}
	else
	{
		// Find connections for each recipient.  We do not transport unicast messages for unknown nodes.
		for (const FGuid& Recipient : InRecipients)
		{
			FWebSocketMessageConnectionRef* RecipientConnection = WebSocketMessageConnections.Find(Recipient);
			if (RecipientConnection && !(*RecipientConnection)->bDestroyed && (*RecipientConnection)->IsConnected())
			{
				RecipientConnections.Add(Recipient, *RecipientConnection);
			}
		}
	}

	if (RecipientConnections.Num() == 0)
	{
		return false;
	}

	const UWebSocketMessagingSettings* Settings = GetDefault<UWebSocketMessagingSettings>();
	
	FWebSocketMessageTransportSerializeHelper::TOnDemandSerializer<FString> JsonSerializer;
	JsonSerializer.bStandardizeCase = Settings->bMessageSerializationStandardizeCase;
	
	FWebSocketMessageTransportSerializeHelper::TOnDemandSerializer<FArrayWriter> CborSerializer;
	
	// Serialize the message on demand in the appropriate format for each peer connections.
	for (const TPair<FGuid, FWebSocketMessageConnectionRef>& Connection : RecipientConnections)
	{
		if (Connection.Value->WebSocketConnection.IsValid())
		{
			// Remark: client connections are always text/json
			if (JsonSerializer.SerializeOnDemand(InContext))
			{
				Connection.Value->WebSocketConnection->Send(JsonSerializer.OutputMessage);
			}
		}
		else if (Connection.Value->WebSocketServerConnection)
		{
			// Remark: server connections are always binary.
			if (Settings->ServerTransportFormat == EWebSocketMessagingTransportFormat::Json)
			{
				if (JsonSerializer.SerializeOnDemand(InContext))
				{
					auto MessageUtf8 = StringCast<UTF8CHAR>(*JsonSerializer.OutputMessage);
					Connection.Value->WebSocketServerConnection->Send(
						reinterpret_cast<const uint8*>(MessageUtf8.Get()), MessageUtf8.Length(), /*bPrependSize*/ false);
				}
			}
			else
			{
				if (CborSerializer.SerializeOnDemand(InContext))
				{
					Connection.Value->WebSocketServerConnection->Send(
						CborSerializer.OutputMessage.GetData(), CborSerializer.OutputMessage.Num(), /*bPrependSize*/ false);
				}
			}
		}
	}

	return true;
}

void FWebSocketMessageTransport::OnConnected(FWebSocketMessageConnectionRef InWebSocketMessageConnection)
{
	UE_LOGF(LogWebSocketMessaging, Log, "Connected to %ls", *InWebSocketMessageConnection->Url);
	InWebSocketMessageConnection->bIsConnecting = false;
}

void FWebSocketMessageTransport::OnServerConnectionClosed(FWebSocketMessageConnectionRef InWebSocketMessageConnection)
{
	UE_LOGF(LogWebSocketMessaging, Log, "%ls disconnected", *InWebSocketMessageConnection->Url);
	ForgetTransportNode(InWebSocketMessageConnection);
	WebSocketMessageConnections.Remove(InWebSocketMessageConnection->Guid);
}

void FWebSocketMessageTransport::RetryConnection(FWebSocketMessageConnectionRef InWebSocketMessageConnection)
{
	if (InWebSocketMessageConnection->bIsConnecting)
	{
		return;
	}

	InWebSocketMessageConnection->RetryHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([](float DeltaTime, FWebSocketMessageConnectionRef WebSocketMessageConnection)
		{
			if (!WebSocketMessageConnection->bDestroyed && !WebSocketMessageConnection->bIsConnecting && !WebSocketMessageConnection->WebSocketConnection->IsConnected())
			{
				WebSocketMessageConnection->bIsConnecting = true;
				WebSocketMessageConnection->WebSocketConnection->Connect();
			}
			return false;
		}, InWebSocketMessageConnection), 1.0f);
}

void FWebSocketMessageTransport::ClientConnected(INetworkingWebSocket* InNetworkingWebSocket)
{
	FString RemoteEndPoint = InNetworkingWebSocket->RemoteEndPoint(true);
	UE_LOGF(LogWebSocketMessaging, Log, "New WebSocket Server connection: %ls", *RemoteEndPoint);

	FGuid Guid = FGuid::NewGuid();

	FWebSocketMessageConnectionRef WebSocketMessageConnection = MakeShared<FWebSocketMessageConnection>(RemoteEndPoint, Guid, InNetworkingWebSocket);

	InNetworkingWebSocket->SetReceiveCallBack(FWebSocketPacketReceivedCallBack::CreateThreadSafeSP(this, &FWebSocketMessageTransport::OnServerJsonMessage, WebSocketMessageConnection));
	InNetworkingWebSocket->SetSocketClosedCallBack(FWebSocketInfoCallBack::CreateThreadSafeSP(this, &FWebSocketMessageTransport::OnServerConnectionClosed, WebSocketMessageConnection));
	InNetworkingWebSocket->SetErrorCallBack(FWebSocketInfoCallBack::CreateThreadSafeSP(this, &FWebSocketMessageTransport::OnServerConnectionClosed, WebSocketMessageConnection));

	WebSocketMessageConnections.Add(Guid, WebSocketMessageConnection);
}

bool FWebSocketMessageTransport::ServerTick(float InDeltaTime)
{
	if (Server.IsValid())
	{
		Server->Tick();
	}

	return true;
}

void FWebSocketMessageTransport::ForgetTransportNode(FWebSocketMessageConnectionRef InWebSocketMessageConnection)
{
	if (TransportHandler)
	{
		TransportHandler->ForgetTransportNode(InWebSocketMessageConnection->Guid);
	}
}
