// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "IMessageTransport.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

struct FWebSocketMessageConnection
{
	FWebSocketMessageConnection() = delete;
	FWebSocketMessageConnection(const FString& InUrl, const FGuid& InGuid, const TSharedRef<class IWebSocket, ESPMode::ThreadSafe>& InWebSocketConnection) :
		Url(InUrl),
		Guid(InGuid),
		WebSocketConnection(InWebSocketConnection),
		WebSocketServerConnection(nullptr),
		bIsConnecting(true),
		bDestroyed(false)
	{

	}

	FWebSocketMessageConnection(const FString& InUrl, const FGuid& InGuid, class INetworkingWebSocket* InWebSocketServerConnection) :
		Url(InUrl),
		Guid(InGuid),
		WebSocketConnection(nullptr),
		WebSocketServerConnection(InWebSocketServerConnection),
		bIsConnecting(true),
		bDestroyed(false)
	{

	}

	bool IsConnected() const;

	void Close();

	/** The WebSocket url */
	const FString Url;

	/** The message transport Guid */
	const FGuid Guid;

	/** Reference to the client websocket connection */
	TSharedPtr<class IWebSocket, ESPMode::ThreadSafe> WebSocketConnection;

	/** Reference to the server websocket connection */
	class INetworkingWebSocket* WebSocketServerConnection;

	/** Is the socket still trying to connect? */
	bool bIsConnecting;

	/** The socket is about to be destroyed */
	bool bDestroyed;

	/** Retry timer */
	FTSTicker::FDelegateHandle RetryHandle;
};

using FWebSocketMessageConnectionRef = TSharedRef<FWebSocketMessageConnection, ESPMode::ThreadSafe>;

class FWebSocketMessageTransport : public IMessageTransport, public TSharedFromThis<FWebSocketMessageTransport>
{
public:
	FWebSocketMessageTransport();
	virtual ~FWebSocketMessageTransport() override;

	//~ Begin IMessageTransport
	virtual FName GetDebugName() const override;
	virtual bool StartTransport(IMessageTransportHandler& InHandler) override;
	virtual void StopTransport() override;
	virtual bool TransportMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext, const TArray<FGuid>& InRecipients) override;
	//~ End IMessageTransport

	bool NeedsRestart() const;

	virtual void OnJsonMessage(const FString& InMessage, FWebSocketMessageConnectionRef InWebSocketMessageConnection);
	virtual void OnConnected(FWebSocketMessageConnectionRef InWebSocketMessageConnection);
	virtual void OnClosed(int32 InCode, const FString& InReason, bool bInUserClose, FWebSocketMessageConnectionRef InWebSocketMessageConnection);
	virtual void OnConnectionError(const FString& InMessage, FWebSocketMessageConnectionRef InWebSocketMessageConnection);

	virtual void RetryConnection(FWebSocketMessageConnectionRef InWebSocketMessageConnection);

	virtual void ClientConnected(class INetworkingWebSocket* InNetworkingWebSocket);

	virtual bool ServerTick(float InDeltaTime);

	virtual void OnServerJsonMessage(void* InData, int32 InDataSize, FWebSocketMessageConnectionRef InWebSocketMessageConnection);
	virtual void OnServerConnectionClosed(FWebSocketMessageConnectionRef InWebSocketMessageConnection);

protected:
	void ForgetTransportNode(FWebSocketMessageConnectionRef InWebSocketMessageConnection);
	
	IMessageTransportHandler* TransportHandler = nullptr;
	TMap<FGuid, FWebSocketMessageConnectionRef> WebSocketMessageConnections;

	TUniquePtr<class IWebSocketServer> Server;
	FTSTicker::FDelegateHandle ServerTickerHandle;

	FString LastServerBindAddress;
	int32 LastServerPort = INDEX_NONE;
	TArray<FString> LastConnectionEndpoints;
	TMap<FString, FString> LastHttpHeaders;
};
