// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebSocketMessagingBeaconReceiver.h"

#include "Backends/CborStructSerializerBackend.h"
#include "CborWriter.h"
#include "Interfaces/IPv4/IPv4Address.h"
#include "JsonObjectConverter.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"
#include "StructSerializer.h"
#include "WebSocketMessagingModule.h"
#include "WebSocketMessagingSettings.h"

namespace UE::WebSocketMessaging::BeaconReceiver::Private
{
	constexpr uint8 ProtocolVersion = 0;
	const TArray<uint8> ProtocolIdentifier = { 'U', 'E', '_', 'W', 'S', 'M', 'B'};

	bool SerializeToJson(const FWebSocketMessagingBeaconPayload& InPayload, FArrayWriter& OutResponseData)
	{
		const UWebSocketMessagingSettings* Settings = GetDefault<UWebSocketMessagingSettings>();
		constexpr int64 CheckFlags = 0;
		constexpr int64 SkipFlags = 0;
		const EJsonObjectConversionFlags ConversionFlags = Settings->bMessageSerializationStandardizeCase ?
			EJsonObjectConversionFlags::None : EJsonObjectConversionFlags::SkipStandardizeCase;

		const TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		
		if (!FJsonObjectConverter::UStructToJsonObject(FWebSocketMessagingBeaconPayload::StaticStruct(), &InPayload, JsonObject,
			CheckFlags, SkipFlags, nullptr, ConversionFlags))
		{
			return false;
		}

		// Todo: see if there is a way to serialize the json object with an utf8 writer directly. 
		FString JsonString;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&JsonString);
		if (!FJsonSerializer::Serialize(JsonObject, Writer))
		{
			return false;
		}

		const auto MessageUtf8 = StringCast<UTF8CHAR>(*JsonString);
		OutResponseData.Serialize(const_cast<UTF8CHAR *>(MessageUtf8.Get()), MessageUtf8.Length());
		return true;
	}

	bool SerializeToCbor(const FWebSocketMessagingBeaconPayload& InPayload, FArrayWriter& OutResponseData)
	{
		FCborWriter CborWriter(&OutResponseData);
		FCborStructSerializerBackend Backend(OutResponseData, EStructSerializerBackendFlags::Default);
		FStructSerializer::Serialize(&InPayload, *FWebSocketMessagingBeaconPayload::StaticStruct(), Backend);
		return true;
	}

}

FWebSocketMessagingBeaconReceiver::FWebSocketMessagingBeaconReceiver()
	: FDiscoveryBeaconReceiver(
		TEXT("WebSocketMessagingBeaconReceiver"),
		UE::WebSocketMessaging::BeaconReceiver::Private::ProtocolIdentifier,
		UE::WebSocketMessaging::BeaconReceiver::Private::ProtocolVersion
	)
{
}

bool FWebSocketMessagingBeaconReceiver::NeedsRestart() const
{
	const UWebSocketMessagingSettings* Settings = GetDefault<UWebSocketMessagingSettings>();
	if (!Settings)
	{
		return false;
	}
	
	if (LastDiscoveryEndpoint != Settings->DiscoveryEndpoint
		|| LastDiscoveryPort != Settings->DiscoveryPort)
	{
		return true;
	}

	return false;
}

void FWebSocketMessagingBeaconReceiver::Startup()
{
	const UWebSocketMessagingSettings& Settings = *GetDefault<UWebSocketMessagingSettings>();
	LastDiscoveryEndpoint = Settings.DiscoveryEndpoint;
	LastDiscoveryPort = Settings.DiscoveryPort;
	
	FDiscoveryBeaconReceiver::Startup();
}

bool FWebSocketMessagingBeaconReceiver::GetDiscoveryAddress(FIPv4Address& OutAddress) const
{
	const UWebSocketMessagingSettings& Settings = *GetDefault<UWebSocketMessagingSettings>();
	FIPv4Address DiscoveryAddress;
	if (!FIPv4Address::Parse(Settings.DiscoveryEndpoint, DiscoveryAddress))
	{
		UE_LOGF(LogWebSocketMessaging, Error, "Failed to parse WebSocket Messaging discovery endpoint address \"%ls\"", *Settings.DiscoveryEndpoint);
		return false;
	}

	OutAddress = DiscoveryAddress;
	return true;
}

int32 FWebSocketMessagingBeaconReceiver::GetDiscoveryPort() const
{
	const UWebSocketMessagingSettings* Settings = GetDefault<UWebSocketMessagingSettings>();
	return Settings ? Settings->DiscoveryPort : 0;
}

bool FWebSocketMessagingBeaconReceiver::MakeBeaconResponse(uint8 InBeaconProtocolVersion, FArrayReader& InMessageData, FArrayWriter& OutResponseData) const
{
	const UWebSocketMessagingSettings* Settings = GetDefault<UWebSocketMessagingSettings>();

	FWebSocketMessagingBeaconPayload Reply;

	if (Settings && Settings->EnableTransport)
	{
		Reply.Services.Reserve(1);
		Reply.Services.Add({FString(TEXT("WebSocketMessaging")), Settings->GetServerPort()});
	}

	using namespace UE::WebSocketMessaging::BeaconReceiver::Private;

	// Json is the default format.
	const EWebSocketMessagingTransportFormat PayloadFormat = Settings ? Settings->DiscoveryPayloadFormat : EWebSocketMessagingTransportFormat::Json; 

	if (PayloadFormat == EWebSocketMessagingTransportFormat::Json)
	{
		return SerializeToJson(Reply, OutResponseData);
	}

	return SerializeToCbor(Reply, OutResponseData);
}
