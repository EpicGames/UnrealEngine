// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DiscoveryBeaconReceiver.h"

#include "WebSocketMessagingBeaconReceiver.generated.h"

/** Service entry */
USTRUCT()
struct FWebSocketMessagingBeaconService
{
	GENERATED_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	int32 Port = 0;
};

/** Payload of the beacon message used as serialization helper. */
USTRUCT()
struct FWebSocketMessagingBeaconPayload
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FWebSocketMessagingBeaconService> Services;
};

/**
 * Receives beacon messages from the External Apps and replies with connection information.
 * This allows the apps to detect compatible Unreal instances on the local network and list them for the user.
 */
class FWebSocketMessagingBeaconReceiver : public FDiscoveryBeaconReceiver
{
public:
	FWebSocketMessagingBeaconReceiver();
	virtual ~FWebSocketMessagingBeaconReceiver() override = default;

	bool NeedsRestart() const;
	
	//~ Begin FDiscoveryBeaconReceiver
	virtual void Startup() override;

protected:
	virtual bool GetDiscoveryAddress(FIPv4Address& OutAddress) const override;
	virtual int32 GetDiscoveryPort() const override;
	virtual bool MakeBeaconResponse(uint8 InBeaconProtocolVersion, FArrayReader& InMessageData, FArrayWriter& OutResponseData) const override;
	//~ End FDiscoveryBeaconReceiver

private:
	FString LastDiscoveryEndpoint;
	int32 LastDiscoveryPort;
};