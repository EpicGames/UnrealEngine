// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebSocketMessagingModule.h"
#include "IMessageBridge.h"
#if WITH_EDITOR
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#endif
#include "MessageBridgeBuilder.h"
#include "WebSocketMessageTransport.h"
#include "WebSocketMessagingBeaconReceiver.h"
#include "WebSocketMessagingSettings.h"

DEFINE_LOG_CATEGORY(LogWebSocketMessaging);

#define LOCTEXT_NAMESPACE "FWebSocketMessagingModule"

void FWebSocketMessagingModule::StartupModule()
{
	if (!FModuleManager::Get().LoadModule(TEXT("WebSockets")))
	{
		return;
	}
#if WITH_EDITOR
	// register settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "WebSocketMessaging",
			LOCTEXT("WebSocketMessagingSettingsName", "WebSocket Messaging"),
			LOCTEXT("WebSocketMessagingSettingsDescription", "Configure the WebSocket Messaging plug-in."),
			GetMutableDefault<UWebSocketMessagingSettings>()
		);

		if (SettingsSection.IsValid())
		{
			SettingsSection->OnModified().BindRaw(this, &FWebSocketMessagingModule::HandleSettingsSaved);
		}
	}
#endif // WITH_EDITOR

	// trigger initialization
	HandleSettingsSaved();
}

void FWebSocketMessagingModule::ShutdownModule()
{
#if WITH_EDITOR
	// unregister settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "WebSocketMessaging");
	}
#endif
}

bool FWebSocketMessagingModule::IsTransportRunning() const
{
	return MessageBridge.IsValid();
}

int32 FWebSocketMessagingModule::GetServerPort() const
{
	if (const UWebSocketMessagingSettings* Settings = GetDefault<UWebSocketMessagingSettings>())
	{
		return Settings->GetServerPort();
	}
	return 0;
}

bool FWebSocketMessagingModule::HandleSettingsSaved()
{
	if (GetDefault<UWebSocketMessagingSettings>()->EnableTransport)
	{
		InitializeBridge();
	}
	else
	{
		ShutdownBridge();
	}

	if (GetDefault<UWebSocketMessagingSettings>()->bEnableDiscoveryListener)
	{
		InitializeBeaconReceiver();
	}
	else
	{
		ShutdownBeaconReceiver();
	}
	return true;
}

void FWebSocketMessagingModule::InitializeBridge()
{
	const TSharedPtr<FWebSocketMessageTransport, ESPMode::ThreadSafe> Transport = TransportWeak.Pin();

	if (Transport && !Transport->NeedsRestart())
	{
		return;
	}

	ShutdownBridge();

	const TSharedRef<FWebSocketMessageTransport, ESPMode::ThreadSafe> NewTransport = MakeShared<FWebSocketMessageTransport, ESPMode::ThreadSafe>();
	TransportWeak = NewTransport;
	MessageBridge = FMessageBridgeBuilder().UsingTransport(NewTransport).Build();
}

void FWebSocketMessagingModule::ShutdownBridge()
{
	if (MessageBridge.IsValid())
	{
		MessageBridge->Disable();
		FPlatformProcess::Sleep(0.1f);
		MessageBridge.Reset();
	}
}

void FWebSocketMessagingModule::InitializeBeaconReceiver()
{
	if (BeaconReceiver && !BeaconReceiver->NeedsRestart())
	{
		return;
	}
	
	ShutdownBeaconReceiver();
	BeaconReceiver = MakeUnique<FWebSocketMessagingBeaconReceiver>();
	BeaconReceiver->Startup();
}

void FWebSocketMessagingModule::ShutdownBeaconReceiver()
{
	if (BeaconReceiver)
	{
		BeaconReceiver->Shutdown();
	}
	BeaconReceiver.Reset();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FWebSocketMessagingModule, WebSocketMessaging)