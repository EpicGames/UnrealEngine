// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWebSocketMessagingModule.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

class FWebSocketMessageTransport;
class FWebSocketMessagingBeaconReceiver;
class UWebSocketMessagingSettings;

DECLARE_LOG_CATEGORY_EXTERN(LogWebSocketMessaging, Log, All);

namespace WebSocketMessaging
{
	namespace Tag
	{
		static constexpr const TCHAR* Sender = TEXT("Sender");
		static constexpr const TCHAR* Recipients = TEXT("Recipients");
		static constexpr const TCHAR* Expiration = TEXT("Expiration");
		static constexpr const TCHAR* TimeSent = TEXT("TimeSent");
		static constexpr const TCHAR* Annotations = TEXT("Annotations");
		static constexpr const TCHAR* Scope = TEXT("Scope");
		static constexpr const TCHAR* MessageType = TEXT("MessageType");
		static constexpr const TCHAR* Message = TEXT("Message");
	}

	namespace Header
	{
		static constexpr const TCHAR* TransportId = TEXT("UE-MessageBus-TransportId");
	}
}

// Todo: implement INetworkMessagingExtension to expose more service controls.
class FWebSocketMessagingModule : public IWebSocketMessagingModule
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface
	
	//~ Begin IWebSocketMessaging
	virtual bool IsTransportRunning() const override;
	virtual int32 GetServerPort() const override;
	//~ End IWebSocketMessaging
	
	virtual bool HandleSettingsSaved();
	
	virtual void InitializeBridge();
	virtual void ShutdownBridge();

	void InitializeBeaconReceiver();
	void ShutdownBeaconReceiver();

protected:
	/** Holds the message bridge if present. */
	TSharedPtr<class IMessageBridge, ESPMode::ThreadSafe> MessageBridge;
	/** Keep track of the transport for access to derived functions. */
	TWeakPtr<FWebSocketMessageTransport, ESPMode::ThreadSafe> TransportWeak;
	/** Multicast Discovery Beacon Receiver, if present. */
	TUniquePtr<FWebSocketMessagingBeaconReceiver> BeaconReceiver;
};