// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebSocketMessagingSettings.h"

#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

int32 UWebSocketMessagingSettings::GetServerPort() const
{
	int32 ServerPortCmdLine;
	// Allow port to be specified on the command line for local server in game mode.
	if (FParse::Value(FCommandLine::Get(), TEXT("-WebSocketMessagingServerPort="), ServerPortCmdLine))
	{
		return ServerPortCmdLine;
	}
	return ServerPort;
}
