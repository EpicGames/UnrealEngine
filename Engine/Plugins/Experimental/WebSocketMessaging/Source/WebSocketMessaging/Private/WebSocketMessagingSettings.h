// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "UObject/Object.h"
#include "WebSocketMessagingSettings.generated.h"

UENUM()
enum class EWebSocketMessagingTransportFormat : uint8
{
	Json,
	Cbor
};

UCLASS(config=Engine)
class UWebSocketMessagingSettings : public UObject
{
	GENERATED_BODY()

public:
	int32 GetServerPort() const;

	/** Whether the WebSocket transport channel is enabled */
	UPROPERTY(config, EditAnywhere, Category=Transport)
	bool EnableTransport = false;

	/**
	 * Bind the WebSocket server on the specified port (0 disables it)
	 * Can be specified on the command line with `-WebSocketMessagingServerPort=`
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport)
	int32 ServerPort = 0;

	/**
	 * The address to bind the websocket server to.
	 * 0.0.0.0 will open the connection to everyone on your network,
	 * while 127.0.0.1 will only allow local requests to come through. 
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport)
	FString ServerBindAddress = TEXT("0.0.0.0");

	/** Format used to serialize the messages on the server's WebSockets.*/
	UPROPERTY(Config, EditAnywhere, Category=Transport)
	EWebSocketMessagingTransportFormat ServerTransportFormat = EWebSocketMessagingTransportFormat::Json;

	/**
	 * For Json formatting only:
	 * If enabled, the "message" part of the messages will have "standardized case" (see FJsonObjectConverter::StandardizeCase).
	 * Mainly, the first character of the field name will be lower case.
	 * If disabled, the field names are not modified.
	 *
	 * For Cbor formatting, the field names are not modified.
	 */ 
	UPROPERTY(Config, EditAnywhere, Category=Transport, meta=(EditCondition="ServerTransportFormat == EWebSocketMessagingTransportFormat::Json"))
	bool bMessageSerializationStandardizeCase = true;
	
    /** The WebSocket Urls to connect to (Eg. ws://example.com/xyz) */
    UPROPERTY(config, EditAnywhere, Category=Transport)
    TArray<FString> ConnectToEndpoints;

	/** Additional HTTP headers to set when connecting to endpoints */
	UPROPERTY(config, EditAnywhere, Category=Transport)
	TMap<FString, FString> HttpHeaders;

	/**
	 * Enables the Multicast Service Discovery
	 */
	UPROPERTY(config, EditAnywhere, Category=Discovery)
	bool bEnableDiscoveryListener = false;

	/**
	 * The IP endpoint to listen to for multicast discovery messages.
	 * The multicast IP address must be in the range 224.0.0.0 to 239.255.255.255.
	 */
	UPROPERTY(config, EditAnywhere, Category=Discovery, meta=(EditCondition="bEnableDiscoveryListener"))
	FString DiscoveryEndpoint = "230.0.0.4";

	/**
	 * The port to listen to for app discovery messages.
	 */
	UPROPERTY(config, EditAnywhere, Category=Discovery, meta=(EditCondition="bEnableDiscoveryListener"))
	int32 DiscoveryPort = 6667;

	/**
	 * Format used to serialize the discovery beacon payload.
	 * The discovery beacon response will have a header followed by the payload.
	 * Header Format: 1 byte Protocol Version, 16 bytes Beacon Guid
	 * The payload is an array of "services" with name and port per service. 
	 */
	UPROPERTY(Config, EditAnywhere, Category=Discovery, meta=(EditCondition="bEnableDiscoveryListener"))
	EWebSocketMessagingTransportFormat DiscoveryPayloadFormat = EWebSocketMessagingTransportFormat::Json;
};
