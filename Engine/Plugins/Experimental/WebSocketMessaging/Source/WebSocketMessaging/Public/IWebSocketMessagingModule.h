// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
 * Interface to provide information on the web socket messaging transport.
 */
class IWebSocketMessagingModule : public IModuleInterface
{
public:
	/** Returns true if the transport is currently running. */
	virtual bool IsTransportRunning() const = 0;
	
	/** Returns the current server port, either from settings or command line. */
	virtual int32 GetServerPort() const = 0;
};