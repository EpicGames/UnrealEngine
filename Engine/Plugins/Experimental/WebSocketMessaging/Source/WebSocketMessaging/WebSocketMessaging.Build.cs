// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WebSocketMessaging : ModuleRules
{
	public WebSocketMessaging(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DiscoveryBeaconReceiver",
				"Engine",
				"Messaging",
				"Networking",
				"WebSockets",
				"WebSocketNetworking",
				"Json",
				"JsonUtilities",
				"Cbor",
				"Serialization"
			}
		);
	}
}
