{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Web Socket Messaging",
	"Description": "Adds a WebSocket based transport layer to the messaging sub-system for sending and receiving messages between networked computers and devices.",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "WebSocketMessaging",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Mac",
				"Win64",
				"Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "DiscoveryBeaconReceiver",
			"Enabled": true
		},
		{
			"Name": "WebSocketNetworking",
			"Enabled": true
		}
	]
}